// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioSettingsEditor : ModuleRules
{
	public AudioSettingsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "Core",
                "InputCore",
				"Engine",
				"Slate",
				"SlateCore",
				
				"PropertyEditor",
				"UnrealEd",
                "CoreUObject"
            }
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"Engine",
			}
		);
	}
}
