// Copyright Epic Games, Inc. All Rights Reserved.

#include "TargetPlatformAudioCustomization.h"

#include "EditorDirectories.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "IDetailChildrenBuilder.h"
#include "UObject/UnrealType.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Features/IModularFeatures.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/SToolTip.h"

#if WITH_ENGINE
#include "AudioDevice.h"
#endif 

IMPLEMENT_MODULE(FDefaultModuleImpl, AudioSettingsEditor)

#define LOCTEXT_NAMESPACE "PlatformAudio"

// This string is used for the item on the combo box that, when selected, defers to the custom string entry.
static const TCHAR* ManualEntryItem = TEXT("Other");

FAudioPluginWidgetManager::FAudioPluginWidgetManager()
{
	ManualReverbEntry = TSharedPtr<FText>(new FText(FText::FromString(TEXT("Built-in Reverb"))));
	ManualSpatializationEntry = TSharedPtr<FText>(new FText(FText::FromString(TEXT("Built-in Spatialization"))));
	ManualOcclusionEntry = TSharedPtr<FText>(new FText(FText::FromString(TEXT("Built-in Occlusion"))));
	ManualSourceDataOverrideEntry = TSharedPtr<FText>(new FText(FText::FromString(TEXT("None"))));
}

FAudioPluginWidgetManager::~FAudioPluginWidgetManager()
{
}

TSharedRef<SWidget> FAudioPluginWidgetManager::MakeAudioPluginSelectorWidget(const TSharedPtr<IPropertyHandle>& PropertyHandle, EAudioPlugin AudioPluginType, const FString& PlatformName)
{
	TArray<TSharedPtr<FText>>* ValidPluginNames = nullptr;
	FText TooltipText;
	TSharedPtr<FText> DefaultEffectName;

	switch (AudioPluginType)
	{
		case EAudioPlugin::SPATIALIZATION:
			ValidPluginNames = &SpatializationPlugins;
			TooltipText = LOCTEXT("Spatialization", "Choose which audio plugin should be used for spatialization. If your desired spatialization isn't found in the drop down menu, ensure that it is enabled on the Plugins panel.");
			DefaultEffectName = TSharedPtr<FText>(new FText(FText::FromString(TEXT("Built-in Spatialization"))));
			SelectedSpatialization = TSharedPtr<FText>(new FText(*DefaultEffectName));
			PropertyHandle->GetValueAsDisplayText(*SelectedSpatialization);
			break;

		case EAudioPlugin::REVERB:
			ValidPluginNames = &ReverbPlugins;
			TooltipText = LOCTEXT("Reverb", "Choose which audio plugin should be used for reverb. If your desired reverb plugin isn't found in the drop down menu, ensure that it is enabled on the Plugins panel.");
			DefaultEffectName = TSharedPtr<FText>(new FText(FText::FromString(TEXT("Built-in Reverb"))));
			SelectedReverb = TSharedPtr<FText>(new FText(*DefaultEffectName));
			PropertyHandle->GetValueAsDisplayText(*SelectedReverb);
			break;

		case EAudioPlugin::OCCLUSION:
			ValidPluginNames = &OcclusionPlugins;
			TooltipText = LOCTEXT("Occlusion", "Choose which audio plugin should be used for occlusion. If your desired occlusion plugin isn't found in the drop down menu, ensure that it is enabled on the Plugins panel.");
			DefaultEffectName = TSharedPtr<FText>(new FText(FText::FromString(TEXT("Built-in Occlusion"))));
			SelectedOcclusion = TSharedPtr<FText>(new FText(*DefaultEffectName));
			PropertyHandle->GetValueAsDisplayText(*SelectedOcclusion);
			break;

		case EAudioPlugin::SOURCEDATAOVERRIDE:
			ValidPluginNames = &SourceDataOverridePlugins;
			TooltipText = LOCTEXT("SourceDataOverride", "Choose which audio plugin should be used for source data override. If your desired source data override plugin isn't found in the drop down menu, ensure that it is enabled on the Plugins panel.");
			DefaultEffectName = TSharedPtr<FText>(new FText(FText::FromString(TEXT("None"))));
			SelectedSourceDataOverride = TSharedPtr<FText>(new FText(*DefaultEffectName));
			PropertyHandle->GetValueAsDisplayText(*SelectedSourceDataOverride);
			break;

		default:
			checkf(false, TEXT("Invalid plugin enumeration type. Need to add a handle for that case here."));
			break;
	}

	ValidPluginNames->Add(DefaultEffectName);
	

#if WITH_ENGINE
	// Scan through all currently enabled audio plugins of this specific type:
	switch (AudioPluginType)
	{
		case EAudioPlugin::SPATIALIZATION:
		{
			IModularFeatures::Get().LockModularFeatureList();
			TArray<IAudioSpatializationFactory*> AvailableSpatializationPlugins = IModularFeatures::Get().GetModularFeatureImplementations<IAudioSpatializationFactory>(IAudioSpatializationFactory::GetModularFeatureName());
			IModularFeatures::Get().UnlockModularFeatureList();

			for (IAudioSpatializationFactory* Plugin : AvailableSpatializationPlugins)
			{
				if (Plugin->SupportsPlatform(PlatformName))
				{
					ValidPluginNames->Add(TSharedPtr<FText>(new FText(FText::FromString(Plugin->GetDisplayName()))));
				}
			}
		}
		break;

		case EAudioPlugin::REVERB:
		{
			IModularFeatures::Get().LockModularFeatureList();
			TArray<IAudioReverbFactory*> AvailableReverbPlugins = IModularFeatures::Get().GetModularFeatureImplementations<IAudioReverbFactory>(IAudioReverbFactory::GetModularFeatureName());
			IModularFeatures::Get().UnlockModularFeatureList();

			for (IAudioReverbFactory* Plugin : AvailableReverbPlugins)
			{
				if (Plugin->SupportsPlatform(PlatformName))
				{
					ValidPluginNames->Add(TSharedPtr<FText>(new FText(FText::FromString(Plugin->GetDisplayName()))));
				}
			}
		}
		break;

		case EAudioPlugin::OCCLUSION:
		{
			IModularFeatures::Get().LockModularFeatureList();
			TArray<IAudioOcclusionFactory*> AvailableOcclusionPlugins = IModularFeatures::Get().GetModularFeatureImplementations<IAudioOcclusionFactory>(IAudioOcclusionFactory::GetModularFeatureName());
			IModularFeatures::Get().UnlockModularFeatureList();

			for (IAudioOcclusionFactory* Plugin : AvailableOcclusionPlugins)
			{
				if (Plugin->SupportsPlatform(PlatformName))
				{
					ValidPluginNames->Add(TSharedPtr<FText>(new FText(FText::FromString(Plugin->GetDisplayName()))));
				}
			}
		}
		break;

		case EAudioPlugin::SOURCEDATAOVERRIDE:
		{
			IModularFeatures::Get().LockModularFeatureList();
			TArray<IAudioSourceDataOverrideFactory*> AvailableSourceDataOverridePlugins = IModularFeatures::Get().GetModularFeatureImplementations<IAudioSourceDataOverrideFactory>(IAudioSourceDataOverrideFactory::GetModularFeatureName());
			IModularFeatures::Get().UnlockModularFeatureList();

			for (IAudioSourceDataOverrideFactory* Plugin : AvailableSourceDataOverridePlugins)
			{
				if (Plugin->SupportsPlatform(PlatformName))
				{
					ValidPluginNames->Add(TSharedPtr<FText>(new FText(FText::FromString(Plugin->GetDisplayName()))));
				}
			}
		}
		break;

		// note: if adding cases here, make sure to lock the Modular Feature List after retrieving the implementations
		//		IModularFeatures::Get().LockModularFeatureList(); & IModularFeatures::Get().UnlockModularFeatureList();

		default:
			IModularFeatures::Get().UnlockModularFeatureList();
			break;
	}
#endif // #if WITH_ENGINE

	// This pointer is used to store whatever custom string was input by the user or retrieved from the config file.
	ValidPluginNames->Add(TSharedPtr<FText>(new FText(FText::FromString(ManualEntryItem))));

	// Text box component:
	TSharedRef<SEditableTextBox> EditableTextBox = SNew(SEditableTextBox)
		.Text_Lambda([this, AudioPluginType]() { return OnGetPluginText(AudioPluginType); })
		.OnTextCommitted_Raw(this, &FAudioPluginWidgetManager::OnPluginTextCommitted, AudioPluginType, PropertyHandle)
		.SelectAllTextWhenFocused(true)
		.RevertTextOnEscape(true);

	// Combo box component:
	TSharedRef<SWidget> ComboBox = SNew(SListView<TSharedPtr<FText>>)
		.ListItemsSource(ValidPluginNames)
		.ScrollbarVisibility(EVisibility::Collapsed)
		.OnGenerateRow_Lambda([](TSharedPtr<FText> InItem, const TSharedRef< class STableViewBase >& Owner)
		{
			return SNew(STableRow<TSharedPtr<FText>>, Owner)
				   .Padding(FMargin(16, 4, 16, 4))
				   [
					   SNew(STextBlock).Text(*InItem)
				   ];
		})
		.OnSelectionChanged_Lambda([this, AudioPluginType, PropertyHandle](TSharedPtr<FText> InText, ESelectInfo::Type)
		{
			const bool bSelectedManualEntry = (InText->ToString() == FString(ManualEntryItem));

			switch (AudioPluginType)
			{
				case EAudioPlugin::SPATIALIZATION:
					if (bSelectedManualEntry)
					{
						SelectedSpatialization = ManualSpatializationEntry;
					}
					else
					{
						SelectedSpatialization = InText;
					}
					
					OnPluginSelected(bSelectedManualEntry ? ManualSpatializationEntry->ToString() : InText->ToString(), PropertyHandle);
					break;

				case EAudioPlugin::REVERB:
					if (bSelectedManualEntry)
					{
						SelectedReverb = ManualReverbEntry;
					}
					else
					{
						SelectedReverb = InText;
					}
					
					OnPluginSelected(bSelectedManualEntry ? ManualReverbEntry->ToString() : InText->ToString(), PropertyHandle);
					break;

				case EAudioPlugin::OCCLUSION:
					if (bSelectedManualEntry)
					{
						SelectedOcclusion = ManualOcclusionEntry;
					}
					else
					{
						SelectedOcclusion = InText;
					}
					
					OnPluginSelected((bSelectedManualEntry ? ManualOcclusionEntry->ToString() : InText->ToString()), PropertyHandle);
					break;

				case EAudioPlugin::SOURCEDATAOVERRIDE:
					if (bSelectedManualEntry)
					{
						SelectedSourceDataOverride = ManualSourceDataOverrideEntry;
					}
					else
					{
						SelectedSourceDataOverride = InText;
					}

					OnPluginSelected((bSelectedManualEntry ? ManualSourceDataOverrideEntry->ToString() : InText->ToString()), PropertyHandle);
					break;

				default:
					break;
			}
		});


	//Generate widget:
	const TSharedRef<SWidget> NewWidget = SNew(SComboButton)
		.ContentPadding(FMargin(0, 0, 5, 0))
		.ToolTipText(TooltipText)
		.ButtonContent()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Padding(FMargin(0, 0, 5, 0))
			[
				EditableTextBox
			]
		]
		.MenuContent()
		[
			ComboBox
		];

	return NewWidget;
}

void FAudioPluginWidgetManager::BuildAudioCategory(IDetailLayoutBuilder& DetailLayout, const FString& PlatformName, const UStruct* ClassOuterMost)
{
	IDetailCategoryBuilder& AudioCategory = DetailLayout.EditCategory(TEXT("Audio"));

	TSharedPtr<IPropertyHandle> AudioSpatializationPropertyHandle = DetailLayout.GetProperty("SpatializationPlugin", ClassOuterMost);
	IDetailPropertyRow& AudioSpatializationPropertyRow = AudioCategory.AddProperty(AudioSpatializationPropertyHandle);

	AudioSpatializationPropertyRow.CustomWidget()
		.NameContent()
		[
			AudioSpatializationPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(500.0f)
		.MinDesiredWidth(100.0f)
		[
			MakeAudioPluginSelectorWidget(AudioSpatializationPropertyHandle, EAudioPlugin::SPATIALIZATION, PlatformName)
		];

	TSharedPtr<IPropertyHandle> AudioReverbPropertyHandle = DetailLayout.GetProperty("ReverbPlugin", ClassOuterMost);
	IDetailPropertyRow& AudioReverbPropertyRow = AudioCategory.AddProperty(AudioReverbPropertyHandle);

	AudioReverbPropertyRow.CustomWidget()
		.NameContent()
		[
			AudioReverbPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(500.0f)
		.MinDesiredWidth(100.0f)
		[
			MakeAudioPluginSelectorWidget(AudioReverbPropertyHandle, EAudioPlugin::REVERB, PlatformName)
		];

	TSharedPtr<IPropertyHandle> AudioOcclusionPropertyHandle = DetailLayout.GetProperty("OcclusionPlugin", ClassOuterMost);
	IDetailPropertyRow& AudioOcclusionPropertyRow = AudioCategory.AddProperty(AudioOcclusionPropertyHandle);

	AudioOcclusionPropertyRow.CustomWidget()
		.NameContent()
		[
			AudioOcclusionPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(500.0f)
		.MinDesiredWidth(100.0f)
		[
			MakeAudioPluginSelectorWidget(AudioOcclusionPropertyHandle, EAudioPlugin::OCCLUSION, PlatformName)
		];
		
		TSharedPtr<IPropertyHandle> AudioPropagationPropertyHandle = DetailLayout.GetProperty("SourceDataOverridePlugin", ClassOuterMost);
		IDetailPropertyRow& AudioPropagationPropertyRow = AudioCategory.AddProperty(AudioPropagationPropertyHandle);

		AudioPropagationPropertyRow.CustomWidget()
			.NameContent()
			[
				AudioPropagationPropertyHandle->CreatePropertyNameWidget()
			]
		.ValueContent()
			.MaxDesiredWidth(500.0f)
			.MinDesiredWidth(100.0f)
			[
				MakeAudioPluginSelectorWidget(AudioPropagationPropertyHandle, EAudioPlugin::SOURCEDATAOVERRIDE, PlatformName)
			];

	// Not really a plugin, but this is common to all TargetPlatforms
	TSharedPtr<IPropertyHandle> SoundQualityNamePropHandle = DetailLayout.GetProperty("SoundCueCookQualityIndex", ClassOuterMost);
	ensure(SoundQualityNamePropHandle.IsValid());
	IDetailPropertyRow& SoundQualityNamePropRow = AudioCategory.AddProperty(SoundQualityNamePropHandle);

	SoundQualityNamePropRow.CustomWidget()
		.NameContent()
		[
			SoundQualityNamePropHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(500.0f)
		.MinDesiredWidth(100.0f)		
		[
			PropertyCustomizationHelpers::MakePropertyComboBox(
				SoundQualityNamePropHandle,
				FOnGetPropertyComboBoxStrings::CreateLambda([](TArray<TSharedPtr<FString>>& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) -> void  
				{									
					for (const FAudioQualitySettings& i : GetDefault<UAudioSettings>()->QualityLevels)
					{
						OutComboBoxStrings.Add(MakeShared<FString>(i.DisplayName.ToString()));
						OutRestrictedItems.Add(false);
						OutToolTips.Add(SNew(SToolTip).Text(i.DisplayName));
					}
				}), 
				FOnGetPropertyComboBoxValue::CreateLambda([SoundQualityNamePropHandle]() -> FString
				{
					int32 IndexVal = INDEX_NONE;
					SoundQualityNamePropHandle->GetValue(IndexVal); 
					return GetDefault<UAudioSettings>()->FindQualityNameByIndex(IndexVal);
				}),
				FOnPropertyComboBoxValueSelected::CreateLambda([SoundQualityNamePropHandle](FString Value) -> void
				{
					const TArray<FAudioQualitySettings>& Values = GetDefault<UAudioSettings>()->QualityLevels;
					int32 Index = Values.IndexOfByPredicate([TextValue = FText::FromString(Value)](const FAudioQualitySettings& i) -> bool { return i.DisplayName.CompareTo(TextValue) == 0; });
					SoundQualityNamePropHandle->SetValue(Index);
				})
			)
		];
}

void FAudioPluginWidgetManager::OnPluginSelected(FString PluginName, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue(PluginName);
}

void FAudioPluginWidgetManager::OnPluginTextCommitted(const FText& InText, ETextCommit::Type CommitType, EAudioPlugin AudioPluginType, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	switch (AudioPluginType)
	{
		case EAudioPlugin::SPATIALIZATION:
			*ManualSpatializationEntry = InText;
			SelectedSpatialization = ManualSpatializationEntry;
			break;

		case EAudioPlugin::REVERB:
			*ManualReverbEntry = InText;
			SelectedReverb = ManualReverbEntry;
			break;

		case EAudioPlugin::OCCLUSION:
			*ManualOcclusionEntry = InText;
			SelectedOcclusion = ManualOcclusionEntry;
			break;

		case EAudioPlugin::SOURCEDATAOVERRIDE:
			*ManualSourceDataOverrideEntry = InText;
			SelectedSourceDataOverride = ManualSourceDataOverrideEntry;
			break;

		default:
			break;
	}

	OnPluginSelected(InText.ToString(), PropertyHandle);
}

FText FAudioPluginWidgetManager::OnGetPluginText(EAudioPlugin AudioPluginType)
{
	switch (AudioPluginType)
	{
		case EAudioPlugin::SPATIALIZATION:
			return *SelectedSpatialization;
	
		case EAudioPlugin::REVERB:
			return *SelectedReverb;

		case EAudioPlugin::OCCLUSION:
			return *SelectedOcclusion;

		case EAudioPlugin::SOURCEDATAOVERRIDE:
			return *SelectedSourceDataOverride;

		default:
			return FText::FromString(FString(TEXT("ERROR")));
	}
}

#undef LOCTEXT_NAMESPACE
