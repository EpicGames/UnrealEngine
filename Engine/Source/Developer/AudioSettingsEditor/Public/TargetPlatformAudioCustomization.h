// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"

#include "AudioPluginUtilities.h"

#define UE_API AUDIOSETTINGSEDITOR_API

class FAudioPluginWidgetManager
{
public:
	UE_API FAudioPluginWidgetManager();
	UE_API ~FAudioPluginWidgetManager();

	/* Builds out the audio category for a specific audio section for a platform settings page. */
	UE_API void BuildAudioCategory(IDetailLayoutBuilder& DetailLayout, const FString& PlatformName, const UStruct* ClassOuterMost = nullptr);

	/** Creates widget from a scan of loaded audio plugins for an individual plugin type. */
	UE_API TSharedRef<SWidget> MakeAudioPluginSelectorWidget(const TSharedPtr<IPropertyHandle>& PropertyHandle, EAudioPlugin AudioPluginType, const FString& PlatformName);

private:
	/** Handles when a new plugin is selected. */
	static UE_API void OnPluginSelected(FString PluginName, TSharedPtr<IPropertyHandle> PropertyHandle);

	UE_API void OnPluginTextCommitted(const FText& InText, ETextCommit::Type CommitType, EAudioPlugin AudioPluginType, TSharedPtr<IPropertyHandle> PropertyHandle);

	UE_API FText OnGetPluginText(EAudioPlugin AudioPluginType);

private:
	/** Cached references to text for Spatialization, Reverb and Occlusion settings */
	TSharedPtr<FText> SelectedReverb;
	TSharedPtr<FText> SelectedSpatialization;
	TSharedPtr<FText> SelectedOcclusion;
	TSharedPtr<FText> SelectedSourceDataOverride;

	TSharedPtr<FText> ManualSpatializationEntry;
	TSharedPtr<FText> ManualReverbEntry;
	TSharedPtr<FText> ManualOcclusionEntry;
	TSharedPtr<FText> ManualSourceDataOverrideEntry;

	TArray<TSharedPtr<FText>> SpatializationPlugins;
	TArray<TSharedPtr<FText>> SourceDataOverridePlugins;
	TArray<TSharedPtr<FText>> ReverbPlugins;
	TArray<TSharedPtr<FText>> OcclusionPlugins;
};

#undef UE_API
